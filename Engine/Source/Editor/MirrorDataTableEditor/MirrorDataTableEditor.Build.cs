// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MirrorDataTableEditor : ModuleRules
{
    public MirrorDataTableEditor(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core", "SlateCore",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
                "CoreUObject",
                "Engine",
                "Slate",
                "SlateCore",
                
                "DataTableEditor",
                "ToolMenus",
                "UnrealEd",
                "AssetTools",
                "EditorStyle",
                
                "SkeletonEditor"
            }
        );
    }
}