// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorDataTableCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "SkeletonTreeBuilder.h"
#include "SkeletonTreeItem.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Styling/AppStyle.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MirrorDataTableCustomization"

/** Denotes a bone item is included in MirrorTable bone scope. */
class FSkeletonTreeMirrorScopeItem : public FSkeletonTreeItem
{
public:
	SKELETON_TREE_ITEM_TYPE(FSkeletonTreeMirrorScopeItem, FSkeletonTreeItem);

	FSkeletonTreeMirrorScopeItem(const TWeakObjectPtr<UMirrorDataTable>& InMirrorDataTable, const FName& InBoneName, bool bInIsVirtualBone, const TSharedRef<ISkeletonTree>& InSkeletonTree)
		: FSkeletonTreeItem(InSkeletonTree)
		, MirrorTable(InMirrorDataTable)
		, bIsVirtualBone(bInIsVirtualBone)
		, BoneName(InBoneName)
	{
	};

	/** ISkeletonTreeItem interface begin */
	virtual void GenerateWidgetForNameColumn(TSharedPtr<SHorizontalBox> Box, const TAttribute<FText>& FilterText, FIsSelected InIsSelected) override
	{
		Box->AddSlot()
		.AutoWidth()
		.Padding(FMargin(0.0f, 1.0f))
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.IsChecked(this, &FSkeletonTreeMirrorScopeItem::GetCheckBoxState)
			.OnCheckStateChanged(this, &FSkeletonTreeMirrorScopeItem::OnCheckBoxStateChanged)
			.ToolTipText(this, &FSkeletonTreeMirrorScopeItem::GetBoneToolTip)
		];

		Box->AddSlot()
		.AutoWidth()
		.Padding(2, 0, 0, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.ColorAndOpacity(this, &FSkeletonTreeMirrorScopeItem::GetBoneTextColor, InIsSelected)
			.Text(FText::FromName(GetRowItemName()))
			.HighlightText(FilterText)
			.Font(this, &FSkeletonTreeMirrorScopeItem::GetBoneTextFont)
			.ToolTipText(this, &FSkeletonTreeMirrorScopeItem::GetBoneToolTip)
		];
	}
	
	virtual TSharedRef<SWidget> GenerateWidgetForDataColumn(const FName& DataColumnName, FIsSelected InIsSelected) override
	{
		return SNullWidget::NullWidget;
	}
	
	virtual FName GetRowItemName() const override
	{
		return BoneName;
	}
	/** ISkeletonTreeItem interface end */

protected:

	ECheckBoxState GetCheckBoxState() const
	{
		return (MirrorTable.IsValid() && MirrorTable->BoneScopeNameList.Contains(BoneName))
			? ECheckBoxState::Checked
			: ECheckBoxState::Unchecked;
	}

	void OnCheckBoxStateChanged(ECheckBoxState NewState)
	{
		if (MirrorTable.IsValid())
		{
			if (NewState == ECheckBoxState::Checked)
			{
				MirrorTable->BoneScopeNameList.Add(BoneName);
			}
			else
			{
				MirrorTable->BoneScopeNameList.Remove(BoneName);
			}
			
			MirrorTable->InvalidateCachedSkeletonData();
		}
	}
	
	FSlateFontInfo GetBoneTextFont() const
	{
		const bool bInScope = MirrorTable.IsValid() && MirrorTable->BoneScopeNameList.Contains(BoneName);
		return bInScope
			? FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.NormalFont").Font
			: FAppStyle::GetWidgetStyle<FTextBlockStyle>("SkeletonTree.ItalicFont").Font;
	}

	FSlateColor GetBoneTextColor(FIsSelected InIsSelected) const
	{
		const bool bInScope = MirrorTable.IsValid() && MirrorTable->BoneScopeNameList.Contains(GetRowItemName());
		if (bInScope)
		{
			return !bIsVirtualBone ? FSlateColor::UseForeground() : FSlateColor(FLinearColor(0.4f, 0.4f, 1.f));
		}
		return !bIsVirtualBone ? FSlateColor::UseSubduedForeground() : FSlateColor(FLinearColor(0.4f, 0.4f, 1.f, 0.5f));
	};
	
	FText GetBoneToolTip() const
	{
		if (MirrorTable.IsValid() && MirrorTable->BoneScopeNameList.Contains(BoneName))
		{
			return LOCTEXT("IncludedBone_ToolTip", "This bone will be included when refreshing the mirror table. Right-click to remove it.");
		}

		return LOCTEXT("ExcludedBone_ToolTip", "This bone will be skipped when refreshing the mirror table. Right-click to include it.");
	}
	
	TWeakObjectPtr<UMirrorDataTable> MirrorTable;

	bool bIsVirtualBone;
	
	/** The name of the item in the tree */
	FName BoneName;
};

/** Same as FSkeletonTreeBuilder but allows me to use FSkeletonTreeMirrorScopeItem. */
class FMirrorDataTableEditorSkeletonTreeBuilder : public FSkeletonTreeBuilder
{
public:
	FMirrorDataTableEditorSkeletonTreeBuilder(const TWeakObjectPtr<UMirrorDataTable>& InMirrorDataTable, const FSkeletonTreeBuilderArgs& InBuilderArgs)
		: FSkeletonTreeBuilder(InBuilderArgs)
		, MirrorTable(InMirrorDataTable)
	{
	}
	
	/** ISkeletonTreeBuilder interface begin */
	virtual void Build(FSkeletonTreeBuilderOutput& Output) override
	{
		TSharedPtr<IEditableSkeleton> EditableSkeleton = EditableSkeletonPtr.Pin();
		if (!EditableSkeleton || !EditableSkeleton->IsSkeletonValid())
		{
			UE_LOGF(LogAnimation, Warning, "Skeleton tree builder - skeleton is invalid");
			return;
		}

		const USkeleton& Skeleton = EditableSkeleton->GetSkeleton();
		const FReferenceSkeleton& RefSkeleton = Skeleton.GetReferenceSkeleton();

		struct FBoneInfo
		{
			FBoneInfo(const FName& InBoneName, const FName& InParentName, int32 InDepth)
				: BoneName(InBoneName)
				, ParentName(InParentName)
				, Depth(InDepth)
			{
				SortString = BoneName.ToString();
				SortNumber = 0;
				SortLength = SortString.Len();

				// Split the bone name into string prefix and numeric suffix for sorting (different from FName to support leading zeros in the numeric suffix)
				int32 Index = SortLength - 1;
				for (int32 PlaceValue = 1; Index >= 0 && FChar::IsDigit(SortString[Index]); --Index, PlaceValue *= 10)
				{
					SortNumber += static_cast<int32>(SortString[Index] - '0') * PlaceValue;
				}
				SortString.LeftInline(Index + 1, EAllowShrinking::No);
			}

			bool operator<(const FBoneInfo& RHS)
			{
				// Sort parents before children
				if (Depth != RHS.Depth)
				{
					return Depth < RHS.Depth;
				}

				// Sort alphabetically by string prefix
				if (int32 SplitNameComparison = SortString.Compare(RHS.SortString))
				{
					return SplitNameComparison < 0;
				}

				// Sort by number if the string prefixes match
				if (SortNumber != RHS.SortNumber)
				{
					return SortNumber < RHS.SortNumber;
				}

				// Sort by length to give us the equivalent to alphabetical sorting if the numbers match (which gives us the following sort order: bone_, bone_0, bone_00, bone_000, bone_001, bone_01, bone_1, etc)
				return (SortNumber == 0) ? SortLength < RHS.SortLength : SortLength > RHS.SortLength;
			}

			FName BoneName = NAME_None;
			FName ParentName = NAME_None;
			int32 Depth = 0;

			FString SortString;
			int32 SortNumber = 0;
			int32 SortLength = 0;
		};

		TArray<FBoneInfo> Bones;
		Bones.Reserve(RefSkeleton.GetRawBoneNum());

		// Gather the bones from the skeleton
		for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetRawBoneNum(); ++BoneIndex)
		{
			const FName& BoneName = RefSkeleton.GetBoneName(BoneIndex);

			FName ParentName = NAME_None;
			int32 ParentIndex = RefSkeleton.GetParentIndex(BoneIndex);
			int32 Depth = 0;
			if (ParentIndex != INDEX_NONE)
			{
				ParentName = RefSkeleton.GetBoneName(ParentIndex);
				Depth = Bones[ParentIndex].Depth + 1;
			}

			Bones.Emplace(BoneName, ParentName, Depth);
		}

		// Sort the bones lexically (and also by depth in order to maintain the invariant of parents before children)
		Algo::Sort(Bones);
		
		// Add the sorted bones to the skeleton tree
		for (const FBoneInfo& Bone : Bones)
		{
			TSharedRef<ISkeletonTreeItem> DisplayBone = MakeShareable(new FSkeletonTreeMirrorScopeItem(MirrorTable, Bone.BoneName, false, SkeletonTreePtr.Pin().ToSharedRef()));
			Output.Add(DisplayBone, Bone.ParentName, FSkeletonTreeMirrorScopeItem::GetTypeId());
		}
		
		// Now add virtual bones.
		const TArray<FVirtualBone>& VirtualBones = EditableSkeleton->GetSkeleton().GetVirtualBones();
		for (const FVirtualBone& VirtualBone : VirtualBones)
		{
			TSharedRef<ISkeletonTreeItem>  DisplayBone = MakeShareable(new FSkeletonTreeMirrorScopeItem(MirrorTable, VirtualBone.VirtualBoneName, true, SkeletonTreePtr.Pin().ToSharedRef()));
			Output.Add(DisplayBone, VirtualBone.SourceBoneName, FSkeletonTreeMirrorScopeItem::GetTypeId(), true);
		}
	}
	/** ISkeletonTreeBuilder interface end */
	
protected:
	
	TWeakObjectPtr<UMirrorDataTable> MirrorTable;
};

TSharedRef<IDetailCustomization> FMirrorDataTableCustomization::MakeInstance()
{
	return MakeShareable(new FMirrorDataTableCustomization);
}

void FMirrorDataTableCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	DetailLayoutBuilder = &DetailBuilder;

	// Get mirror table being edited/customized
	{
		TArray<TWeakObjectPtr<UObject>> Objects;
		DetailBuilder.GetObjectsBeingCustomized(Objects);

		if (Objects.Num() > 0)
		{
			MirrorTable = Cast<UMirrorDataTable>(Objects[0].Get());
		}
	}

	// Setup skeleton tree widget
	{
		FSkeletonTreeBuilderArgs BuilderArgs;
		BuilderArgs.bShowBones = true;
		BuilderArgs.bShowSockets = false;
		BuilderArgs.bShowAttachedAssets = false;
		BuilderArgs.bShowVirtualBones = true;
		
		FSkeletonTreeArgs TreeArgs;
		TreeArgs.Mode = ESkeletonTreeMode::Picker;
		TreeArgs.bAllowMeshOperations = false;
		TreeArgs.bAllowSkeletonOperations = false;
		TreeArgs.bShowBlendProfiles = false;
		TreeArgs.bShowFilterMenu = false;
		TreeArgs.bHideBonesByDefault = false;
		TreeArgs.bAllowInvisibleItemSelection = true;
		TreeArgs.ContextName = TEXT("MirrorDataTableCustomization");
		TreeArgs.Builder = SkeletonTreeBuilder = MakeShared<FMirrorDataTableEditorSkeletonTreeBuilder>(MirrorTable, BuilderArgs);
		TreeArgs.Extenders = MakeShared<FExtender>();
		TreeArgs.Extenders->AddMenuExtension("SkeletonTreeContextMenu", EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& InMenuBuilder)
			{
				// Returns the given bone and all its descendants from the skeleton hierarchy.
				// Works for both regular and virtual bones since both are present in FReferenceSkeleton.
				auto GetBoneAndDescendants = [this](FName BoneName) -> TArray<FName>
				{
					TArray<FName> Result;
					if (!MirrorTable.IsValid() || !MirrorTable->Skeleton)
					{
						return Result;
					}

					const FReferenceSkeleton& RefSkel = MirrorTable->Skeleton->GetReferenceSkeleton();
					const int32 RootIndex = RefSkel.FindBoneIndex(BoneName);
					if (RootIndex == INDEX_NONE)
					{
						return Result;
					}

					Result.Add(BoneName);

					const int32 NumBones = RefSkel.GetNum();
					for (int32 i = RootIndex + 1; i < NumBones; ++i)
					{
						int32 ParentIndex = RefSkel.GetParentIndex(i);
						while (ParentIndex != INDEX_NONE)
						{
							if (ParentIndex == RootIndex)
							{
								Result.Add(RefSkel.GetBoneName(i)); break;
							}
							
							if (ParentIndex < RootIndex)
							{
								break;
							}
							
							ParentIndex = RefSkel.GetParentIndex(ParentIndex);
						}
					}
						
					return Result;
				};

				InMenuBuilder.BeginSection("MirrorTableScope", LOCTEXT("MirrorTableScopeSection", "Scope"));

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("MenuLabel", "Include bone"),
					LOCTEXT("MenuTooltip", "This bone will be included the next time the mirror table is refreshed from the skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return;
							}
								
							FScopedTransaction Transaction(LOCTEXT("IncludeBone", "Include bone when refreshing"));

							MirrorTable->Modify();

							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								MirrorTable->BoneScopeNameList.Add(SelectedItem->GetRowItemName());
							}
								
							MirrorTable->InvalidateCachedSkeletonData();
						}),
						FCanExecuteAction::CreateLambda([this]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return false;
							}
								
							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								if (!MirrorTable->BoneScopeNameList.Contains(SelectedItem->GetRowItemName()))
								{
									return true;
								}
							}
								
							return false;
						})
					)
				);

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("RemoveMenuLabel", "Exclude bone"),
					LOCTEXT("RemoveMenuTooltip", "This bone will be skipped the next time the mirror table is refreshed from the skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return;
							}
								
							FScopedTransaction Transaction(LOCTEXT("ExcludeBone", "Exclude bone when refreshing"));

							MirrorTable->Modify();

							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								MirrorTable->BoneScopeNameList.Remove(SelectedItem->GetRowItemName());
							}
								
							MirrorTable->InvalidateCachedSkeletonData();
						}),
						FCanExecuteAction::CreateLambda([this]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return false;
							}
								
							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								if (MirrorTable->BoneScopeNameList.Contains(SelectedItem->GetRowItemName()))
								{
									return true;
								}
							}
								
							return false;
						})
					)
				);

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("IncludeWithChildrenLabel", "Include bone and children"),
					LOCTEXT("IncludeWithChildrenTooltip", "This bone and all its children will be included the next time the mirror table is refreshed from the skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, GetBoneAndDescendants]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return;
							}
								
							FScopedTransaction Transaction(LOCTEXT("IncludeBoneAndChildren", "Include bone and children when refreshing"));

							MirrorTable->Modify();

							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								for (const FName& BoneName : GetBoneAndDescendants(SelectedItem->GetRowItemName()))
								{
									MirrorTable->BoneScopeNameList.Add(BoneName);
								}
							}
								
							MirrorTable->InvalidateCachedSkeletonData();
						}),
						FCanExecuteAction::CreateLambda([this, GetBoneAndDescendants]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return false;
							}
								
							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								for (const FName& BoneName : GetBoneAndDescendants(SelectedItem->GetRowItemName()))
								{
									if (!MirrorTable->BoneScopeNameList.Contains(BoneName))
									{
										return true;
									}
								}
							}
								
							return false;
						})
					)
				);

				InMenuBuilder.AddMenuEntry(
					LOCTEXT("ExcludeWithChildrenLabel", "Exclude bone and children"),
					LOCTEXT("ExcludeWithChildrenTooltip", "This bone and all its children will be excluded the next time the mirror table is refreshed from the skeleton."),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([this, GetBoneAndDescendants]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return;
							}
								
							FScopedTransaction Transaction(LOCTEXT("ExcludeBoneAndChildren", "Exclude bone and children when refreshing"));

							MirrorTable->Modify();

							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								for (const FName& BoneName : GetBoneAndDescendants(SelectedItem->GetRowItemName()))
								{
									MirrorTable->BoneScopeNameList.Remove(BoneName);
								}
							}
								
							MirrorTable->InvalidateCachedSkeletonData();
						}),
						FCanExecuteAction::CreateLambda([this, GetBoneAndDescendants]()
						{
							if (!MirrorTable.IsValid() || !SkeletonTree.IsValid())
							{
								return false;
							}
								
							for (TSharedPtr<ISkeletonTreeItem> SelectedItem : SkeletonTree->GetSelectedItems())
							{
								for (const FName& BoneName : GetBoneAndDescendants(SelectedItem->GetRowItemName()))
								{
									if (MirrorTable->BoneScopeNameList.Contains(BoneName))
									{
										return true;
									}
								}
							}
							return false;
						})
					)
				);

				InMenuBuilder.EndSection();
			})
		);

		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");

		if (MirrorTable.IsValid() && MirrorTable->Skeleton)
		{
			SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(MirrorTable->Skeleton, TreeArgs);
		}
	}
	
	// Add info icon to MirrorFindReplaceExpressions property (regex run order).
	{
		IDetailCategoryBuilder& CreateTableCategory = DetailBuilder.EditCategory("CreateTable");
		TSharedPtr<IPropertyHandle> FindReplaceHandle = DetailBuilder.GetProperty(
			GET_MEMBER_NAME_CHECKED(UMirrorDataTable, MirrorFindReplaceExpressions));

		if (FindReplaceHandle.IsValid())
		{
			IDetailPropertyRow& Row = CreateTableCategory.AddProperty(FindReplaceHandle);

			static const FText InfoText = LOCTEXT("FindReplaceOrderInfo",
				"Expressions are evaluated from top to bottom. The first matching expression is used.");

			Row.CustomWidget(true)
			.NameContent()
			[
				SNew(SHorizontalBox)
				.ToolTipText(InfoText)

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					FindReplaceHandle->CreatePropertyNameWidget()
				]

				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(8.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Info"))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
			];
		}
	}

	// Pin the 'Import' category to the bottom of the details panel
	IDetailCategoryBuilder& ImportCategory = DetailBuilder.EditCategory("Import");
	ImportCategory.SetSortOrder(INT_MAX);

	BoneScopeHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UMirrorDataTable, BoneScope));

	// Add the BoneScope enum dropdown
	ImportCategory.AddProperty(BoneScopeHandle);
	
	// Refresh the panel when enum changes so the next row updates immediately
	BoneScopeHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]()
		{
			if (DetailLayoutBuilder)
			{
				DetailLayoutBuilder->ForceRefreshDetails();
			}
		})
	);
	
	// Show skeleton tree bone widget only for specific bone selection
	ImportCategory.AddCustomRow(LOCTEXT("SpecificBoneListFilter", "Specific Bone List"))
	.Visibility(TAttribute<EVisibility>::CreateLambda([this]()
		{
			if (!BoneScopeHandle.IsValid())
			{
				return EVisibility::Collapsed;
			}
			
			uint8 RawValue = 0;
			if (BoneScopeHandle->GetValue(RawValue) != FPropertyAccess::Success)
			{
				return EVisibility::Collapsed;
			}
			
			const EMirrorTableBoneRefreshScope Scope = static_cast<EMirrorTableBoneRefreshScope>(RawValue);
			return Scope == EMirrorTableBoneRefreshScope::ExplicitBoneList ? EVisibility::Visible : EVisibility::Collapsed;
		})
	)
	.WholeRowContent()
	[
		SNew(SBox)
		.Padding(FMargin(-8.f, 0.f, 0.f, 0.f))
		[
			SNew(SExpandableArea)
			.InitiallyCollapsed(false)
			.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
			.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
			.HeaderContent()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SpecificBoneListLabel", "Specific Bones"))
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(6.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						if (!MirrorTable.IsValid() || MirrorTable->BoneScopeNameList.Num() == 0)
						{
							return LOCTEXT("SelectedBoneCountEmpty", "(none — add bones below)");
						}
						return FText::Format(LOCTEXT("SelectedBoneCount", "({0} selected)"), MirrorTable->BoneScopeNameList.Num());
					})
					.Font(IDetailLayoutBuilder::GetDetailFontItalic())
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.f)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("SelectAllBonesLabel", "Select All"))
					.ToolTipText(LOCTEXT("SelectAllBonesTooltip", "Add all bones from the skeleton to the list."))
					.IsEnabled_Lambda([this]()
					{
						if (!MirrorTable.IsValid() || !MirrorTable->Skeleton)
						{
							return false;
						}
						const int32 TotalBones = MirrorTable->Skeleton->GetReferenceSkeleton().GetRawBoneNum();
						return MirrorTable->BoneScopeNameList.Num() < TotalBones;
					})
					.OnClicked_Lambda([this]()
					{
						if (MirrorTable.IsValid() && MirrorTable->Skeleton)
						{
							const FReferenceSkeleton& RefSkeleton = MirrorTable->Skeleton->GetReferenceSkeleton();
							for (int32 i = 0; i < RefSkeleton.GetRawBoneNum(); ++i)
							{
								MirrorTable->BoneScopeNameList.Add(RefSkeleton.GetBoneName(i));
							}
							MirrorTable->InvalidateCachedSkeletonData();
						}
						return FReply::Handled();
					})
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.Text(LOCTEXT("ClearAllBonesLabel", "Clear All"))
					.ToolTipText(LOCTEXT("ClearAllBonesTooltip", "Clear the bone list and switch back to using the full skeleton."))
					.OnClicked_Lambda([this]()
					{
						if (MirrorTable.IsValid() && BoneScopeHandle.IsValid())
						{
							MirrorTable->BoneScopeNameList.Empty();
							MirrorTable->InvalidateCachedSkeletonData();
							
							BoneScopeHandle->SetValue(static_cast<uint8>(EMirrorTableBoneRefreshScope::FullSkeleton));
						}
						return FReply::Handled();
					})
				]
			]
			.BodyContent()
			[
				SNew(SBox)
				.MinDesiredWidth(300.f)
				.MaxDesiredHeight(600.f)
				[
					SNew(SScrollBox)
					+ SScrollBox::Slot()
					[
						SkeletonTree.IsValid()
						? StaticCastSharedRef<SWidget>(SkeletonTree.ToSharedRef())
						: StaticCastSharedRef<SWidget>(SNew(STextBlock).Text(LOCTEXT("NoSkeleton", "No skeleton available.")))
					]
				]
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE