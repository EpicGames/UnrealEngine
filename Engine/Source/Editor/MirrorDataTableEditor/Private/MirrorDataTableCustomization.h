// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Animation/MirrorDataTable.h"

class FSkeletonTreeBuilder;
class ISkeletonTree;
class IDetailLayoutBuilder;
class SWidget;
class IPropertyHandle;

class FMirrorDataTableCustomization : public IDetailCustomization
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	// End of IDetailCustomization interface

protected:
	
	TSharedPtr<FSkeletonTreeBuilder> SkeletonTreeBuilder;
	TSharedPtr<ISkeletonTree> SkeletonTree;
	TWeakObjectPtr<UMirrorDataTable> MirrorTable;
	TSharedPtr<IPropertyHandle> BoneScopeHandle;
	IDetailLayoutBuilder* DetailLayoutBuilder = nullptr;
};