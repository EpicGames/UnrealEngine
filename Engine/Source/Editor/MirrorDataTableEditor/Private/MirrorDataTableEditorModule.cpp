// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorDataTableEditorModule.h"

#include "DataTableEditorModule.h"
#include "DataTableEditorUtils.h"
#include "DataTableRowMenuContext.h"
#include "IEditableSkeleton.h"
#include "ISkeletonEditorModule.h"
#include "MirrorDataTableCustomization.h"
#include "MirrorDataTableValidator.h"
#include "PropertyEditorModule.h"
#include "ScopedTransaction.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Animation/AnimationSettings.h"
#include "Animation/AnimCurveMetadata.h"
#include "Animation/MirrorDataTable.h"
#include "Framework/Commands/Commands.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Logging/StructuredLog.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "FMirrorDataTableEditorModule"

/** Returns the mirror data table being currently edited, if any. */
static UMirrorDataTable* GetMirrorDataTableFromContext(const FToolMenuContext& Context)
{
	if (const UAssetEditorToolkitMenuContext* ToolkitMenuContext = Context.FindContext<UAssetEditorToolkitMenuContext>())
	{
		if (ToolkitMenuContext && ToolkitMenuContext->Toolkit.IsValid())
		{
			for (UObject* Object : ToolkitMenuContext->GetEditingObjects())
			{
				if (Object && Object->IsA<UMirrorDataTable>())
				{
					return Cast<UMirrorDataTable>(Object);
				}
			}
		}
	}

	return nullptr;
}

/** Returns the selected row in the mirror data table currently being edited, if any. */
static FMirrorTableRow* GetMirrorTableRowFromContext(const FToolMenuContext& Context)
{
	UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context);
	if (!MirrorTable)
	{
		return nullptr;
	}

	const UDataTableRowMenuContext* RowCtx = Context.FindContext<UDataTableRowMenuContext>();
	if (!RowCtx || !RowCtx->RowDataPtr)
	{
		return nullptr;
	}

	return MirrorTable->FindRow<FMirrorTableRow>(RowCtx->RowDataPtr->RowId, TEXT(""));
}

void FMirrorDataTableEditorModule::StartupModule()
{
	// Add mirror table specific menus to data table editor toolbar.
    UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FMirrorDataTableEditorModule::RegisterMenus));

	// Register customization for mirror table.
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyModule.RegisterCustomClassLayout("MirrorDataTable", FOnGetDetailCustomizationInstance::CreateStatic(&FMirrorDataTableCustomization::MakeInstance));

	// Bind immediately — does not require GEditor.
	OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FMirrorDataTableEditorModule::OnPropertyChanged);

	// UAssetEditorSubsystem requires GEditor, which is not yet valid at StartupModule time. Defer until PostEngineInit.
	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			if (GEditor)
			{
				if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
				{
					OnAssetOpenedHandle = AssetEditorSubsystem->OnAssetOpenedInEditor().AddRaw(this, &FMirrorDataTableEditorModule::OnAssetOpenedInEditor);
				}
			}
		}
	);
}

void FMirrorDataTableEditorModule::ShutdownModule()
{
    UToolMenus::UnRegisterStartupCallback(this);

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("MirrorDataTable");
	}

	// Unbind all active skeleton callbacks.
	TArray<TWeakObjectPtr<UMirrorDataTable>> Keys;
	ActiveSkeletonCallbacks.GetKeys(Keys);
	for (const TWeakObjectPtr<UMirrorDataTable>& WeakMirrorTable : Keys)
	{
		if (UMirrorDataTable* MirrorTable = WeakMirrorTable.Get())
		{
			UnbindSkeletonCallbacks(MirrorTable);
		}
	}
	ActiveSkeletonCallbacks.Empty();

	// Unbind global delegates.
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);

	if (GEditor)
	{
		if (UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			AssetEditorSubsystem->OnAssetOpenedInEditor().Remove(OnAssetOpenedHandle);
		}
	}
}

void FMirrorDataTableEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);
	
	if (UToolMenu* MirrorToolBarMenu = UToolMenus::Get()->ExtendMenu("AssetEditor.DataTableEditor.ToolBar"))
	{
		MirrorToolBarMenu->AddDynamicSection("MirrorDataTableDynamicSection", FNewSectionConstructChoice(
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* InMenu)
				{
					if (!InMenu)
					{
						return;
					}

					// Don't extend anything if we're editing any other data table type.
					UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(InMenu->Context);
					if (!MirrorTable)
					{
						return;
					}
				
					// Our commands will go here.
					FToolMenuSection& Section = InMenu->FindOrAddSection("MirrorDataTable");

					// Do a full sync/reimport from skeleton.
					FToolMenuEntry& ReimportFromSkeleton = Section.AddEntry(FToolMenuEntry::InitToolBarButton(
						"MDT_ReimportFromSkeleton",
						FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
							{
								ExecuteMenuAction_ReimportFromSkeleton(Context, UMirrorDataTable::FFindReplaceOptions::Sync().WithNotification());
							}
						),
						LOCTEXT("ReimportLabel", "Reimport from Skeleton"),
						TAttribute<FText>::CreateLambda([MirrorTable]() -> FText
						{
							static const FText OperationDescription = LOCTEXT("ReimportOperationDescription", "Syncs the table with the skeleton: adds missing entries, updates existing ones, and disables stale entries.");
								
							if (!MirrorTable)
							{
								return OperationDescription;
							}
								
							switch (MirrorTable->GetSkeletonSyncStatus())
							{
							case UMirrorDataTable::ESyncStatus::Stale:
								return FText::Format(LOCTEXT("ReimportToolTip_Stale", "(Out of sync): {0}"), OperationDescription);
							case UMirrorDataTable::ESyncStatus::UpToDate:
								return FText::Format(LOCTEXT("ReimportToolTip_UpToDate", "(Up to date): {0}"), OperationDescription);
							default:
								return FText::Format(LOCTEXT("ReimportToolTip_Unknown", "(Unknown): {0}"), OperationDescription);
							}
						}),
						TAttribute<FSlateIcon>::CreateStatic(&FMirrorDataTableEditorModule::GetStatusIcon, MirrorTable)
					));
					ReimportFromSkeleton.StyleNameOverride = FName("CalloutToolbar");
					
					// Reimport from skeleton partially.
					FToolMenuEntry& ReimportFromSkeletonOptions = Section.AddEntry(FToolMenuEntry::InitComboButton(
						"MDT_ReimportFromSkeletonOptions",
						FUIAction(),
						FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([this](const FToolMenuContext& Context)
							{
								static const FName MenuName = "AssetEditor.MirrorDataTableEditor.ReimportFromSkeletonMenu";

								if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
								{
									UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
									FToolMenuSection& ActionsSection = Menu->FindOrAddSection("Modes", LOCTEXT("ModesLabel", "Actions"));

									// Only add missing entries.
									ActionsSection.AddMenuEntry(
										NAME_None,
										LOCTEXT("AddMissingLabel", "Add Missing Entries"),
										LOCTEXT("AddMissingTooltip", "Only add rows for skeleton entries that do not yet exist in the mirror data table."),
										FSlateIcon(),
										FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
											{
												ExecuteMenuAction_ReimportFromSkeleton(Context,
													UMirrorDataTable::FFindReplaceOptions::AddMissingOnly().WithNotification()
												);
											}
										)
									);

									// Update existing entries.
									ActionsSection.AddMenuEntry(
										NAME_None,
										LOCTEXT("UpdateExistingEntriesLabel", "Update Existing Entries"),
										LOCTEXT("UpdateExistingEntriesTooltip", "Only update existing rows."),
										FSlateIcon(),
										FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
											{
												ExecuteMenuAction_ReimportFromSkeleton(Context,
													UMirrorDataTable::FFindReplaceOptions::UpdateExisting().WithNotification()
												);
											}
										)
									);
									
									ActionsSection.AddMenuEntry(
										NAME_None,
										LOCTEXT("DisableStaleEntriesLabel", "Disable Stale Entries"),
										LOCTEXT("DisableStaleEntriesToolTip", "Only disables entries found in table but not in skeleton."),
										FSlateIcon(),
										FToolMenuExecuteAction::CreateLambda([](const FToolMenuContext& Context)
											{
												UMirrorDataTable::FFindReplaceOptions Options;
												Options.Flags |= UMirrorDataTable::FFindReplaceOptions::EFlags::DisableStaleRows;
												Options.bShowNotification = true;
												
												ExecuteMenuAction_ReimportFromSkeleton(Context, Options);
											}
										)
									);
									
									FToolMenuSection& OptionsSection = Menu->FindOrAddSection("Options", LOCTEXT("OptionsLabel", "Options"));
									
									OptionsSection.AddMenuEntry(
										NAME_None,
										LOCTEXT("MapToSelfLabel", "Default to self on find/replace failure"),
										LOCTEXT("MapToSelfTooltip", "Maps entry name to itself when the find/replace fails to find a mirror entry name."),
										FSlateIcon(),
										FUIAction(
											FExecuteAction::CreateLambda([]()
												{
													if (UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
													{
														AnimationSettings->bOnMirrorPairFindReplaceFailedMapToSelf = !AnimationSettings->bOnMirrorPairFindReplaceFailedMapToSelf;
													}
												}
											),
											FCanExecuteAction(),
											FIsActionChecked::CreateLambda([]()
												{
													if (UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
													{
														return AnimationSettings->bOnMirrorPairFindReplaceFailedMapToSelf;
													}
													
													return false;
												}
											)
										),
										EUserInterfaceActionType::Check
									);
								}

								return UToolMenus::Get()->GenerateWidget(MenuName, Context);
							})
						),
						FText::GetEmpty(),
						LOCTEXT("ReimportFromSkeletonOptions", "Choose a reimport action")
					));
					ReimportFromSkeletonOptions.StyleNameOverride = FName("CalloutToolbar");
					ReimportFromSkeletonOptions.ToolBarData.bSimpleComboBox = true;;
				
					// Validate and inform user.
					FToolMenuEntry& Validate = Section.AddEntry(FToolMenuEntry::InitToolBarButton(
						"MDT_Validate",
						FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_Validate),
						LOCTEXT("ValidateLabel", "Validate"),
						LOCTEXT("ValidateTooltip", "Validate mirror data table is properly set up."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Validate")
					));
					Validate.StyleNameOverride = FName("CalloutToolbar");
					
					// Validation options.
					FToolMenuEntry& ValidateOptions = Section.AddEntry(FToolMenuEntry::InitComboButton(
						"MDT_ValidateOptions",
						FUIAction(),
						FNewToolMenuChoice(FNewToolMenuWidget::CreateLambda([](const FToolMenuContext& Context)
							{
								static const FName MenuName = "AssetEditor.MirrorDataTableEditor.ValidateMenu";

								if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
								{
									UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(MenuName);
									FToolMenuSection& OptionsSection = Menu->FindOrAddSection("Options", LOCTEXT("ValidateOptionsLabel", "Options"));

									OptionsSection.AddMenuEntry(
										NAME_None,
										LOCTEXT("ValidateOnOpenLabel", "Validate on Open"),
										LOCTEXT("ValidateOnOpenTooltip", "Automatically validate the mirror data table when it is opened in the editor. Only shows a notification if issues are found."),
										FSlateIcon(),
										FUIAction(
											FExecuteAction::CreateLambda([]()
												{
													if (UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
													{
														AnimationSettings->bValidateMirrorDataTableOnOpen = !AnimationSettings->bValidateMirrorDataTableOnOpen;
													}
												}
											),
											FCanExecuteAction(),
											FIsActionChecked::CreateLambda([]()
												{
													if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
													{
														return AnimationSettings->bValidateMirrorDataTableOnOpen;
													}

													return false;
												}
											)
										),
										EUserInterfaceActionType::Check
									);
								}

								return UToolMenus::Get()->GenerateWidget(MenuName, Context);
							})
						),
						FText::GetEmpty(),
						LOCTEXT("ValidateOptionsTooltip", "Validation options")
					));
					ValidateOptions.StyleNameOverride = FName("CalloutToolbar");
					ValidateOptions.ToolBarData.bSimpleComboBox = true;
				
					// Only update existing entries (no stale entries).
					Section.AddEntry(FToolMenuEntry::InitToolBarButton(
						"MDT_AutoPair",
						FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_AutoPair),
						LOCTEXT("AutoPairLabel", "Auto Pair"),
						LOCTEXT("AutoPairTooltip", "Auto Pair using Find Replace Expressions"),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AutomationTools.MenuIcon")
					));

					// Clear all rows from the mirror data table.
					Section.AddEntry(FToolMenuEntry::InitToolBarButton(
						"MDT_ClearAll",
						FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_ClearSheet),
						LOCTEXT("ClearAllLabel", "Clear Sheet"),
						LOCTEXT("ClearAllTooltip", "Remove all rows from the mirror data table."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Delete")
					));
					
					// @todo: Preview changes option. Useful for big skeletons like meta human.
				})
			));
	}

	// Add mirror data table specific actions to the row right-click menu.
	if (UToolMenu* RowContextMenu = UToolMenus::Get()->ExtendMenu("DataTableEditor.RowContextMenu"))
	{
		RowContextMenu->AddDynamicSection("MirrorDataTableRowDynamicSection", FNewSectionConstructChoice(
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					if (!InMenu)
					{
						return;
					}

					// Only add mirror-specific row actions when editing a MirrorDataTable.
					if (!GetMirrorDataTableFromContext(InMenu->Context))
					{
						return;
					}

					const FMirrorTableRow* Row = GetMirrorTableRowFromContext(InMenu->Context);
					const bool bIsEnabled = Row ? Row->bEnabled : false;

					FToolMenuSection& Section = InMenu->FindOrAddSection("MirrorRowActions", LOCTEXT("MirrorRowActionsLabel", "Mirror"));

					// Toggle this row on/off without removing it from the table.
					{
						const FText EnableLabel = LOCTEXT("EnableRowLabel", "Enable Row");
						const FText DisableLabel = LOCTEXT("DisableRowLabel", "Disable Row");
						const FText EnableTooltip = LOCTEXT("EnableRowTooltip", "Mark this entry as active so it is used when mirroring animations.");
						const FText DisableTooltip = LOCTEXT("DisableRowTooltip", "Mark this entry as inactive. It stays in the table but won't be used when mirroring animations.");
						const FSlateIcon EnableIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Visible");
						const FSlateIcon DisableIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Hidden");

						Section.AddMenuEntry(
							"MDT_ToggleRowEnabled",
							bIsEnabled ? DisableLabel : EnableLabel,
							bIsEnabled ? DisableTooltip : EnableTooltip,
							bIsEnabled ? DisableIcon : EnableIcon,
							FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_ToggleRowEnabled)
						);
					}

					// Swap which name is the source and which is the mirror.
					Section.AddMenuEntry(
						"MDT_SwapPair",
						LOCTEXT("SwapPairLabel", "Swap Mirror Pair"),
						LOCTEXT("SwapPairTooltip", "Swap the source and mirror names for this entry."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "UMGEditor.Mirror"),
						FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_SwapPair)
					);

					// Auto-fill the mirrored name using the table's naming rules.
					Section.AddMenuEntry(
						"MDT_AutoPairRow",
						LOCTEXT("AutoPairRowLabel", "Auto Pair Row"),
						LOCTEXT("AutoPairRowTooltip", "Automatically fill in the mirrored name for this entry using the table's find/replace expressions."),
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "AutomationTools.MenuIcon"),
						FToolMenuExecuteAction::CreateStatic(&FMirrorDataTableEditorModule::ExecuteMenuAction_AutoPairRow)
					);
				})
			));
	}
}

void FMirrorDataTableEditorModule::BindSkeletonCallbacks(UMirrorDataTable* MirrorTable)
{
	if (!MirrorTable || !MirrorTable->Skeleton)
	{
		return;
	}

	// Unbind any existing callbacks first (e.g. skeleton changed, we're rebinding to a new one).
	UnbindSkeletonCallbacks(MirrorTable);

	ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
	TSharedPtr<IEditableSkeleton> EditableSkeleton = SkeletonEditorModule.CreateEditableSkeleton(MirrorTable->Skeleton);

	FSkeletonCallbackHandles Handles;
	Handles.EditableSkeleton = EditableSkeleton;
	Handles.BoundSkeleton = MirrorTable->Skeleton;

	// Listen for bone hierarchy changes. Only sync if the data is actually stale to avoid redundant work.
	MirrorTable->Skeleton->RegisterOnSkeletonHierarchyChanged(USkeleton::FOnSkeletonHierarchyChanged::CreateWeakLambda(MirrorTable, [MirrorTable]()
		{
			if (!MirrorTable->Skeleton)
			{
				return;
			}

			if (MirrorTable->GetSkeletonSyncStatus() == UMirrorDataTable::ESyncStatus::UpToDate)
			{
				return;
			}

			UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Skeleton hierarchy changed. Updating entries.", *MirrorTable->GetName());
			MirrorTable->UpdateFromFindReplaceExpressions(UMirrorDataTable::FFindReplaceOptions::Sync());
		}
	));

	// Listen for notifies and/or sync marker changes.
	if (EditableSkeleton && EditableSkeleton->IsSkeletonValid())
	{
		EditableSkeleton->RegisterOnNotifiesChanged(FSimpleMulticastDelegate::FDelegate::CreateWeakLambda(MirrorTable, [MirrorTable]()
			{
				UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Skeleton notifies and/or sync markers changed. Updating entries.", *MirrorTable->GetName());
				MirrorTable->UpdateFromFindReplaceExpressions(UMirrorDataTable::FFindReplaceOptions::Sync());
			}
		));
	}

	// Listen for curve name changes.
	if (UAnimCurveMetaData* AnimCurveMetaData = MirrorTable->Skeleton->GetAssetUserData<UAnimCurveMetaData>())
	{
		Handles.CurveMetaDataChangedHandle = AnimCurveMetaData->RegisterOnCurveMetaDataChanged(FSimpleMulticastDelegate::FDelegate::CreateWeakLambda(MirrorTable, [MirrorTable]()
			{
				UE_LOGF(LogAnimation, Log, "MirrorDataTable '%ls': Skeleton curve names changed. Updating entries.", *MirrorTable->GetName());
				MirrorTable->UpdateFromFindReplaceExpressions(UMirrorDataTable::FFindReplaceOptions::Sync());
			}
		));
	}

	ActiveSkeletonCallbacks.Add(MirrorTable, MoveTemp(Handles));
}

void FMirrorDataTableEditorModule::UnbindSkeletonCallbacks(UMirrorDataTable* MirrorTable)
{
	FSkeletonCallbackHandles* Handles = ActiveSkeletonCallbacks.Find(MirrorTable);
	if (!Handles)
	{
		return;
	}

	// Use BoundSkeleton, not the mirror table's Skeleton, since the Skeleton property may have already changed by the time we unbind.
	if (USkeleton* BoundSkeleton = Handles->BoundSkeleton.Get())
	{
		BoundSkeleton->UnregisterOnSkeletonHierarchyChanged(MirrorTable);

		if (Handles->EditableSkeleton)
		{
			Handles->EditableSkeleton->UnregisterOnNotifiesChanged(MirrorTable);
		}

		if (Handles->CurveMetaDataChangedHandle.IsValid())
		{
			if (UAnimCurveMetaData* AnimCurveMetaData = BoundSkeleton->GetAssetUserData<UAnimCurveMetaData>())
			{
				AnimCurveMetaData->UnregisterOnCurveMetaDataChanged(Handles->CurveMetaDataChangedHandle);
			}
		}
	}

	ActiveSkeletonCallbacks.Remove(MirrorTable);
}

void FMirrorDataTableEditorModule::OnAssetOpenedInEditor(UObject* Asset, IAssetEditorInstance* EditorInstance)
{
	UMirrorDataTable* MirrorTable = Cast<UMirrorDataTable>(Asset);
	if (!MirrorTable)
	{
		return;
	}

	BindSkeletonCallbacks(MirrorTable);

	if (const UAnimationSettings* AnimationSettings = UAnimationSettings::Get())
	{
		if (AnimationSettings->bValidateMirrorDataTableOnOpen)
		{
			// Only notify if issues are found — clean pass is silent to avoid noise.
			ValidateMirrorDataTable(MirrorTable, false);
		}
	}
}

void FMirrorDataTableEditorModule::OnPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent)
{
	UMirrorDataTable* MirrorTable = Cast<UMirrorDataTable>(Object);
	if (!MirrorTable)
	{
		return;
	}

	const FName MemberPropertyName = PropertyChangedEvent.GetMemberPropertyName();

	// Skeleton property changed - rebind to the new skeleton.
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMirrorDataTable, Skeleton))
	{
		BindSkeletonCallbacks(MirrorTable);
	}
	// Bone scope settings changed - mark table as stale.
	else if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMirrorDataTable, BoneScope) ||
	         MemberPropertyName == GET_MEMBER_NAME_CHECKED(UMirrorDataTable, BoneScopeNameList))
	{
		MirrorTable->InvalidateCachedSkeletonData();
	}
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_ReimportFromSkeleton(const FToolMenuContext& Context, const UMirrorDataTable::FFindReplaceOptions& Options)
{
	if (UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context))
	{
		FScopedTransaction Transaction(LOCTEXT("UndoAction_ReimportFromSkeleton", "Re-import missing data from the associated Skeleton."));
		
		MirrorTable->UpdateFromFindReplaceExpressions(Options);
		
		MirrorTable->MarkPackageDirty();
	}
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_Validate(const FToolMenuContext& Context)
{
	if (UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context))
	{
		ValidateMirrorDataTable(MirrorTable, true);
	}
}

void FMirrorDataTableEditorModule::ValidateMirrorDataTable(UMirrorDataTable* MirrorTable, bool bNotifyOnCleanPass)
{
	const FMirrorDataTableValidator::FResult Result = FMirrorDataTableValidator::Validate(*MirrorTable);

	uint32 NumErrors = 0;
	uint32 NumWarnings = 0;

	for (const FMirrorDataTableValidator::FIssue& Issue : Result.Issues)
	{
		const TCHAR* LogFormatString = !Issue.RowName.IsNone() ? TEXT("MirrorDataTable '{0}': Row '{1}': {2}") : TEXT("MirrorDataTable '{0}': {2}");
		const FString FormattedString = FString::Format(LogFormatString, { MirrorTable->GetName(), Issue.RowName.ToString(), Issue.Message.ToString() });

		switch (Issue.Severity)
		{
		case FMirrorDataTableValidator::FIssue::ESeverity::Info:
			UE_LOGF(LogAnimation, Log, "%ls", *FormattedString);
			break;
		case FMirrorDataTableValidator::FIssue::ESeverity::Warning:
			UE_LOGF(LogAnimation, Warning, "%ls", *FormattedString);
			++NumWarnings;
			break;
		case FMirrorDataTableValidator::FIssue::ESeverity::Error:
			UE_LOGF(LogAnimation, Error, "%ls", *FormattedString);
			++NumErrors;
			break;
		}
	}

	const bool bHasIssues = NumErrors > 0 || NumWarnings > 0;
	if (!bHasIssues && !bNotifyOnCleanPass)
	{
		return;
	}

	const FText CleanPassedSummary = LOCTEXT("PassedSummary_Validate", "Validation passed.");
	const FText PassWithWarningsSummary = FText::Format(LOCTEXT("PassedWithWarningsSummary_Validate", "Validation passed with {0} warnings. See log for details."), NumWarnings);
	const FText FinalPassedSummary = NumWarnings == 0 ? CleanPassedSummary : PassWithWarningsSummary;
	const FText FailedSummary = FText::Format(LOCTEXT("FailedSummary_Validate", "Validation failed. Found {0} errors and {1} warnings. See log for details."), NumErrors, NumWarnings);
	const FText Summary = NumErrors <= 0 ? FinalPassedSummary : FailedSummary;

	FNotificationInfo Info = FNotificationInfo(Summary);
	Info.bFireAndForget = true;
	Info.ExpireDuration = 4.0f;
	Info.bUseSuccessFailIcons = true;

	if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(Info))
	{
		Notification->SetCompletionState(NumErrors > 0 ? SNotificationItem::CS_Fail : SNotificationItem::CS_Success);
	}
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_AutoPair(const FToolMenuContext& Context)
{
	if (UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context))
	{
		FScopedTransaction Transaction(LOCTEXT("UndoAction_AutoPairMirrorEntries", "Automatically find mirror pairs for all entries in table."));
		
		UMirrorDataTable::FFindReplaceOptions Options { UMirrorDataTable::FFindReplaceOptions::EFlags::UpdateExistingRows, true};
		MirrorTable->UpdateFromFindReplaceExpressions(Options);
		
		MirrorTable->MarkPackageDirty();
	}
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_ClearSheet(const FToolMenuContext& Context)
{
	if (UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context))
	{
		FScopedTransaction Transaction(LOCTEXT("UndoAction_ClearSheet", "Clear all rows from the mirror data table."));
		
		MirrorTable->Modify();
		
		// Clear sheet and let others know we've done so.
		MirrorTable->EmptyTable();
		MirrorTable->InvalidateCachedSkeletonData();
		MirrorTable->OnDataTableChanged().Broadcast();
		
		// Let editor know we've changed the data and refresh view.
		FDataTableEditorUtils::BroadcastPostChange(MirrorTable, FDataTableEditorUtils::EDataTableChangeInfo::RowList);
		
		MirrorTable->MarkPackageDirty();
	}
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_ToggleRowEnabled(const FToolMenuContext& Context)
{
	UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context);
	const UDataTableRowMenuContext* RowCtx = Context.FindContext<UDataTableRowMenuContext>();
	if (!MirrorTable || !RowCtx || !RowCtx->RowDataPtr)
	{
		return;
	}

	FMirrorTableRow* Row = MirrorTable->FindRow<FMirrorTableRow>(RowCtx->RowDataPtr->RowId, TEXT(""));
	if (!Row)
	{
		return;
	}

	FScopedTransaction Transaction(Row->bEnabled ? LOCTEXT("UndoAction_DisableRow", "Disable Row") : LOCTEXT("UndoAction_EnableRow", "Enable Row"));
	MirrorTable->Modify();

	// Toggle enabled state and rebuild runtime mirror maps.
	Row->bEnabled = !Row->bEnabled;
	MirrorTable->OnDataTableChanged().Broadcast();

	// Let editor know we've changed the data and refresh view.
	FDataTableEditorUtils::BroadcastPostChange(MirrorTable, FDataTableEditorUtils::EDataTableChangeInfo::RowData);
	MirrorTable->MarkPackageDirty();
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_SwapPair(const FToolMenuContext& Context)
{
	UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context);
	const UDataTableRowMenuContext* RowCtx = Context.FindContext<UDataTableRowMenuContext>();
	if (!MirrorTable || !RowCtx || !RowCtx->RowDataPtr)
	{
		return;
	}

	FMirrorTableRow* Row = MirrorTable->FindRow<FMirrorTableRow>(RowCtx->RowDataPtr->RowId, TEXT(""));
	if (!Row)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("UndoAction_SwapPair", "Swap Mirror Pair"));
	MirrorTable->Modify();

	// Swap source and mirror names, then rebuild runtime mirror maps.
	Swap(Row->Name, Row->MirroredName);
	MirrorTable->OnDataTableChanged().Broadcast();

	// Let editor know we've changed the data and refresh view.
	FDataTableEditorUtils::BroadcastPostChange(MirrorTable, FDataTableEditorUtils::EDataTableChangeInfo::RowData);
	MirrorTable->MarkPackageDirty();
}

void FMirrorDataTableEditorModule::ExecuteMenuAction_AutoPairRow(const FToolMenuContext& Context)
{
	UMirrorDataTable* MirrorTable = GetMirrorDataTableFromContext(Context);
	const UDataTableRowMenuContext* RowCtx = Context.FindContext<UDataTableRowMenuContext>();
	if (!MirrorTable || !RowCtx || !RowCtx->RowDataPtr)
	{
		return;
	}

	FMirrorTableRow* Row = MirrorTable->FindRow<FMirrorTableRow>(RowCtx->RowDataPtr->RowId, TEXT(""));
	if (!Row)
	{
		return;
	}

	// Derive the mirrored name using the table's naming rules. No-op if no match is found.
	const FName MirroredName = MirrorTable->FindReplace(Row->Name);
	if (MirroredName.IsNone())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("UndoAction_AutoPairRow", "Auto Pair Row"));
	MirrorTable->Modify();

	Row->MirroredName = MirroredName;
	MirrorTable->OnDataTableChanged().Broadcast();

	// Let editor know we've changed the data and refresh view.
	FDataTableEditorUtils::BroadcastPostChange(MirrorTable, FDataTableEditorUtils::EDataTableChangeInfo::RowData);
	MirrorTable->MarkPackageDirty();
}

FSlateIcon FMirrorDataTableEditorModule::GetStatusIcon(UMirrorDataTable* InMirrorDataTable)
{
	static const FName ReimportStatusBackground("Persona.AssetClass.Skeleton");
	static const FName ReimportStatusUnknown("Blueprint.CompileStatus.Overlay.Unknown");
	static const FName ReimportStatusError("Blueprint.CompileStatus.Overlay.Error");
	static const FName ReimportStatusGood("Blueprint.CompileStatus.Overlay.Good");
	static const FName ReimportStatusWarning("Blueprint.CompileStatus.Overlay.Warning");
	
	if (InMirrorDataTable)
	{
		if (!InMirrorDataTable->Skeleton)
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), ReimportStatusBackground, NAME_None, ReimportStatusError);
		}
	
		switch (InMirrorDataTable->GetSkeletonSyncStatus())
		{
		case UMirrorDataTable::ESyncStatus::UpToDate:
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), ReimportStatusBackground, NAME_None, ReimportStatusGood);
		case UMirrorDataTable::ESyncStatus::Stale:
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), ReimportStatusBackground, NAME_None, ReimportStatusWarning);
		default:
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), ReimportStatusBackground, NAME_None, ReimportStatusUnknown);
		}
	}

	return FSlateIcon(FAppStyle::GetAppStyleSetName(), ReimportStatusBackground, NAME_None, ReimportStatusUnknown);
}

#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(FMirrorDataTableEditorModule, MirrorDataTableEditor)