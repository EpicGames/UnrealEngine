// Copyright Epic Games, Inc. All Rights Reserved.

#include "MirrorDataTableValidator.h"

#include "Animation/Skeleton.h"
#include "Misc/ScopedSlowTask.h"

#define LOCTEXT_NAMESPACE "MDT_Validator"

constexpr int32 NumSkeletonBackedRowTypes = 4;
constexpr int32 NumRowTypes = 5;

static TStaticArray<FText, NumRowTypes> RowTypeLabels { 
	LOCTEXT("BoneLabel", "Bone"),
	LOCTEXT("AnimNotifyLabel", "Animation Notify"),
	LOCTEXT("CurveLabel", "Curve"),
	LOCTEXT("SyncMarkerLabel", "Sync Marker"),
	LOCTEXT("CustomLabel", "Custom"),
};

FMirrorDataTableValidator::FResult FMirrorDataTableValidator::Validate(const UMirrorDataTable& MirrorDataTable)
{
	FResult OutResult;

	// Keep track of progress.
	constexpr int32 NumProgressSteps = 3;
	FScopedSlowTask Progress(NumProgressSteps, LOCTEXT("ValidatorProgress", "Validating Mirror Data Table..."));
	
	// Show progress dialog.
    constexpr bool bShowCancelButton = true;
    Progress.MakeDialog(bShowCancelButton);
	
	// Warn if bone scope is set to explicit but no bones are selected.
	if (MirrorDataTable.BoneScope == EMirrorTableBoneRefreshScope::ExplicitBoneList && MirrorDataTable.BoneScopeNameList.Num() == 0)
	{
		OutResult.AddIssue(
			FIssue::ESeverity::Warning,
			LOCTEXT("EmptyBoneScopeList", "Bone scope is set to 'Explicit Bone List' but no bones are selected. The full skeleton will be used as a fallback when refreshing.")
		);
	}

	ValidateSkeletonBackedEntries(MirrorDataTable, OutResult, Progress);
	ValidateDuplicateMappings(MirrorDataTable, OutResult, Progress);
	ValidateReverseMappings(MirrorDataTable, OutResult, Progress);
	
	return OutResult;
}

void FMirrorDataTableValidator::ValidateSkeletonBackedEntries(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress)
{
	InOutProgress.EnterProgressFrame(1.0f, LOCTEXT("ValidateSkeletonEntries", "Checking for missing or stale skeleton entries..."));
	
	if (!MirrorDataTable.Skeleton)
	{
		OutResult.AddIssue(
			FIssue::ESeverity::Error,
			LOCTEXT("MissingSkeleton", "Mirror Data Table has no associated skeleton.")
		);
		return;
	}
	
	TSet<FName> SourceEntriesPerCategory[NumSkeletonBackedRowTypes];
	TSet<FName> DisabledEntriesPerCategory[NumSkeletonBackedRowTypes];
	
	TArray<FName> SkeletonCurveNames;
	MirrorDataTable.Skeleton->GetCurveMetaDataNames(SkeletonCurveNames);
	
	auto DoesSkeletonEntryExist = [&SkeletonCurveNames](const UMirrorDataTable& MirrorDataTable, EMirrorRowType::Type RowType, FName EntryName) -> bool
	{
		if (EntryName.IsNone() || !MirrorDataTable.Skeleton)
		{
			return false;
		}
			
		switch (RowType)
		{
		case EMirrorRowType::Bone:
			return MirrorDataTable.Skeleton->GetReferenceSkeleton().FindBoneIndex(EntryName) != INDEX_NONE;
			
		case EMirrorRowType::AnimationNotify:
			return MirrorDataTable.Skeleton->AnimationNotifies.Contains(EntryName);
			
		case EMirrorRowType::Curve:
			return SkeletonCurveNames.Contains(EntryName);
			
		case EMirrorRowType::SyncMarker:
			return MirrorDataTable.Skeleton->GetExistingMarkerNames().Contains(EntryName);
			
		case EMirrorRowType::Custom:
		default: 
			return false;
		}
	};
	
	MirrorDataTable.ForeachRow<FMirrorTableRow>(TEXT("ValidateSkeletonEntries"), [&DoesSkeletonEntryExist, &MirrorDataTable, &OutResult, &SourceEntriesPerCategory, &DisabledEntriesPerCategory](const FName & RowName, const FMirrorTableRow& Row)
	{
		// Custom row entry does not rely on skeleton. Skip validation.
		if (Row.MirrorEntryType == EMirrorRowType::Custom)
		{
			return;
		}

		// Determine severity.
		FIssue::ESeverity IssueSeverity = Row.bEnabled ? FIssue::ESeverity::Error : FIssue::ESeverity::Warning;
		FText DisabledSuffix;
			
		// Don't fail validation when disabled.
		if (!Row.bEnabled)
		{
			DisabledSuffix = LOCTEXT("DisabledEntry", " This entry is disabled.");
			
			DisabledEntriesPerCategory[Row.MirrorEntryType].Add(Row.Name);
		}
			
		// Check for entry being invalid.
		if (Row.Name.IsNone())
		{
			OutResult.AddIssue(
				FIssue::ESeverity::Error,
				LOCTEXT("InvalidSourceEntry", "Entry not specified."),
				RowName
			);
		}
			
		// Always track the source entry so the second pass doesn't falsely report it as missing.
		SourceEntriesPerCategory[Row.MirrorEntryType].Add(Row.Name);

		// Check for mirror entry being invalid.
		if (Row.MirroredName.IsNone())
		{
			OutResult.AddIssue(
				IssueSeverity,
				FText::Format(LOCTEXT("InvalidMirroredEntry", "Mirrored entry not specified.{0}"), DisabledSuffix),
				RowName
			);
		}
			
		// Stale entries (missing in skeleton, found in table)
			
		// Check for row entry name being stale
		if (!Row.Name.IsNone() && !DoesSkeletonEntryExist(MirrorDataTable, Row.MirrorEntryType, Row.Name))
		{
			OutResult.AddIssue(
				IssueSeverity,
				FText::Format(LOCTEXT("MissingSourceEntry", "{1} '{0}' no longer exists in skeleton data.{2}"), FText::FromName(Row.Name), RowTypeLabels[Row.MirrorEntryType], DisabledSuffix),
				RowName
			);
		}
			
		// Check for row mirror pair name being stale
		if (!Row.MirroredName.IsNone() && !DoesSkeletonEntryExist(MirrorDataTable, Row.MirrorEntryType, Row.MirroredName))
		{
			OutResult.AddIssue(
				IssueSeverity,
				FText::Format(LOCTEXT("MissingMirroredEntry", "Mirrored {1} '{0}' no longer exists in skeleton data.{2}"), FText::FromName(Row.MirroredName), RowTypeLabels[Row.MirrorEntryType], DisabledSuffix),
				RowName
			);
		}
	});
	
	// Missing entries (missing in table, found in skeleton)
	{
		auto ProcessEntry = [&DisabledEntriesPerCategory, &OutResult, &SourceEntriesPerCategory](EMirrorRowType::Type RowType, FName EntryName)
		{
			if (!SourceEntriesPerCategory[RowType].Contains(EntryName))
			{
				const bool bIsEnabled = !DisabledEntriesPerCategory[RowType].Contains(EntryName);
				FIssue::ESeverity IssueSeverity = bIsEnabled ? FIssue::ESeverity::Error : FIssue::ESeverity::Warning;
				FText DisabledSuffix = !bIsEnabled ? LOCTEXT("DisabledEntry", " This entry is disabled.") : FText::GetEmpty();


				OutResult.AddIssue(
					IssueSeverity,
					FText::Format(LOCTEXT("MissingEntry", "{0} '{1}' is missing from the table.{2}"), RowTypeLabels[RowType], FText::FromName(EntryName), DisabledSuffix)
				);
			}
		};
		
		const FReferenceSkeleton& ReferenceSkeleton = MirrorDataTable.Skeleton->GetReferenceSkeleton();
		if (MirrorDataTable.BoneScope == EMirrorTableBoneRefreshScope::ExplicitBoneList && MirrorDataTable.BoneScopeNameList.Num() > 0)
		{
			for (const FName& Name : MirrorDataTable.BoneScopeNameList)
			{
				if (ReferenceSkeleton.FindBoneIndex(Name) != INDEX_NONE)
				{
					ProcessEntry(EMirrorRowType::Bone, Name);
				}
			}
		}
		else
		{
			for (int32 BoneIndex = 0; BoneIndex < ReferenceSkeleton.GetNum(); ++BoneIndex)
			{
				const FName Name = ReferenceSkeleton.GetBoneName(BoneIndex);
				ProcessEntry(EMirrorRowType::Bone, Name);
			}
		}
		
		for (const FName& Name : MirrorDataTable.Skeleton->AnimationNotifies)
		{
			ProcessEntry(EMirrorRowType::AnimationNotify, Name);
		}
		
		for (const FName& Name : MirrorDataTable.Skeleton->GetExistingMarkerNames())
		{
			ProcessEntry(EMirrorRowType::SyncMarker, Name);
		}
		
		for (const FName& Name : SkeletonCurveNames)
		{
			ProcessEntry(EMirrorRowType::Curve, Name);
		}
	}
}

void FMirrorDataTableValidator::ValidateDuplicateMappings(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress)
{
	InOutProgress.EnterProgressFrame(1.0f, LOCTEXT("ValidateDuplicateMappings", "Checking for duplicate entries..."));
	
	TMap<FName, TArray<FName>> CategoryMappings[NumRowTypes]; // [row type] entry name -> array of row names
	
	MirrorDataTable.ForeachRow<FMirrorTableRow>(TEXT("ValidateDuplicateMappings"), [&MirrorDataTable, &OutResult, &CategoryMappings](const FName & RowName, const FMirrorTableRow& Row)
	{
		TArray<FName>& RowNames = CategoryMappings[Row.MirrorEntryType].FindOrAdd(Row.Name);
		RowNames.Add(RowName);
	});
	
	// For each row type
	for (int CategoryIndex = 0; CategoryIndex < NumRowTypes; CategoryIndex++)
	{
		// Check if any entry names are found more than once
		for (const TTuple<FName, TArray<FName>>& EntryDuplicates : CategoryMappings[CategoryIndex])
		{
			const TArray<FName> & RowNames = EntryDuplicates.Value;
			if (RowNames.Num() <= 1)
			{
				continue;
			}
			
			// Log an issue for each duplicate found
			const FName EntryName = EntryDuplicates.Key;
			for (const FName& RowName : RowNames)
			{
				OutResult.AddIssue(
					FIssue::ESeverity::Error,
					FText::Format(LOCTEXT("DuplicateMappings", "Found duplicate mappings for entry '{0}'."), FText::FromName(EntryName)),
					RowName
				);
			}
		}
	}
}

void FMirrorDataTableValidator::ValidateReverseMappings(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress)
{
	InOutProgress.EnterProgressFrame(1.0f, LOCTEXT("ValidateReverseMappings", "Checking that mirroring is invertible..."));
	
	TMap<FName, FName> EntryNameToRowName[NumRowTypes];
	
	MirrorDataTable.ForeachRow<FMirrorTableRow>(TEXT("ValidateReverseMappings"), [&MirrorDataTable, &OutResult, &EntryNameToRowName](const FName & RowName, const FMirrorTableRow& Row)
	{
		EntryNameToRowName[Row.MirrorEntryType].FindOrAdd(Row.Name) = RowName;
	});
	
	const bool bUsingExplicitBoneList = MirrorDataTable.BoneScope == EMirrorTableBoneRefreshScope::ExplicitBoneList && MirrorDataTable.BoneScopeNameList.Num() > 0;

	MirrorDataTable.ForeachRow<FMirrorTableRow>(TEXT("ValidateReverseMappings"), [&MirrorDataTable, &OutResult, &EntryNameToRowName, bUsingExplicitBoneList](const FName & RowName, const FMirrorTableRow& Row)
	{
		FName MirroredRowName = EntryNameToRowName[Row.MirrorEntryType].FindRef(Row.MirroredName);
		const FMirrorTableRow* MirroredRow = MirrorDataTable.FindRow<FMirrorTableRow>(MirroredRowName, TEXT("ValidateReverseMappings"), false);

		FIssue::ESeverity IssueSeverity = !Row.bEnabled ? FIssue::ESeverity::Warning : FIssue::ESeverity::Error;
		FText DisabledSuffix = !Row.bEnabled ? LOCTEXT("DisabledEntry", " This entry is disabled.") : FText::GetEmpty();

		// When using an explicit bone list, skip reverse mapping validation for bone rows where either side is out of scope.
		// Bones that mirror to nothing still fall through to the regular checks below.
		if (bUsingExplicitBoneList && Row.MirrorEntryType == EMirrorRowType::Bone && !Row.MirroredName.IsNone())
		{
			const bool bSourceInScope = MirrorDataTable.BoneScopeNameList.Contains(Row.Name);
			const bool bMirrorInScope = MirrorDataTable.BoneScopeNameList.Contains(Row.MirroredName);

			if (!bSourceInScope || !bMirrorInScope)
			{
				if (bSourceInScope && !bMirrorInScope)
				{
					OutResult.AddIssue(
						FIssue::ESeverity::Info,
						FText::Format(
							LOCTEXT("MirrorTargetOutOfScope", "Bone '{0}' is in the bone scope list but its mirror target '{1}' is not. The reverse entry may not be updated on the next refresh.{2}"),
							FText::FromName(Row.Name),
							FText::FromName(Row.MirroredName),
							DisabledSuffix
						),
						RowName
					);
				}
				return;
			}
		}

		if (!MirroredRow && !Row.MirroredName.IsNone())
		{
			OutResult.AddIssue(
				IssueSeverity,
				FText::Format(
					LOCTEXT("MissingInvertibleMapping", "{0} '{1}' mirrors to '{2}', but '{2}' does not have a matching reverse entry.{3}"), 
					RowTypeLabels[Row.MirrorEntryType],
					FText::FromName(Row.Name),
					FText::FromName(Row.MirroredName),
					DisabledSuffix
				),
				RowName
			);
		}
			
		if (MirroredRow)
		{
			if (MirroredRow->MirroredName != Row.Name)
			{
				OutResult.AddIssue(
					IssueSeverity,
					FText::Format(
						LOCTEXT("NonInvertibleMapping", "{0} '{1}' mirrors to '{2}', but '{2}' mirrors to '{3}' instead."), 
						RowTypeLabels[Row.MirrorEntryType],
						FText::FromName(Row.Name),
						FText::FromName(Row.MirroredName),
						FText::FromName(MirroredRow->MirroredName)
					),
					RowName
				);
			}
			else if (Row.bEnabled && !MirroredRow->bEnabled)
			{
				OutResult.AddIssue(
					FIssue::ESeverity::Error,
					FText::Format(
						LOCTEXT("DisabledReverseMapping", "{0} '{1}' mirrors to '{2}', but the matching reverse entry '{3}' is disabled."), 
						RowTypeLabels[Row.MirrorEntryType],
						FText::FromName(Row.Name),
						FText::FromName(Row.MirroredName),
						FText::FromName(MirroredRowName)
					),
					RowName
				);
			}
		}
	});
}

#undef LOCTEXT_NAMESPACE