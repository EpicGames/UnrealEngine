// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/MirrorDataTable.h"
#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"

class IAssetEditorInstance;
class IEditableSkeleton;
class USkeleton;
class UMirrorDataTable;
struct FToolMenuContext;

class FMirrorDataTableEditorModule : public IModuleInterface
{
public:

	/** Begin IModuleInterface */
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;
	/** End IModuleInterface */

	/** Register tool menus specific when editing a mirror data tables */
    void RegisterMenus();

	/** Reimports and syncs the active mirror data table from its skeleton. */
	static void ExecuteMenuAction_ReimportFromSkeleton(const FToolMenuContext& Context, const UMirrorDataTable::FFindReplaceOptions& Options);

	/** Validates the active mirror data table. */
	static void ExecuteMenuAction_Validate(const FToolMenuContext& Context);

	/** Auto-generates mirror pairs for the active mirror data table. */
	static void ExecuteMenuAction_AutoPair(const FToolMenuContext& Context);

	/** Clears all rows from the active mirror data table. */
	static void ExecuteMenuAction_ClearSheet(const FToolMenuContext& Context);

	/** Toggles the enabled state of the selected row in the active mirror data table. */
	static void ExecuteMenuAction_ToggleRowEnabled(const FToolMenuContext& Context);

	/** Swaps the Name and MirroredName of the selected row. */
	static void ExecuteMenuAction_SwapPair(const FToolMenuContext& Context);

	/** Applies find/replace expressions to the selected row to derive its mirrored name. */
	static void ExecuteMenuAction_AutoPairRow(const FToolMenuContext& Context);

	/** Returns the status icon for the given mirror data table. */
    static FSlateIcon GetStatusIcon(UMirrorDataTable* InMirrorDataTable);

private:

	/** Validates the given mirror data table and logs all issues. Skips showing notification on validation passed if requested. */
	static void ValidateMirrorDataTable(UMirrorDataTable* MirrorTable, bool bNotifyOnCleanPass);

	/** Stores per-mirror-table skeleton callback handles for an actively-edited mirror data table. */
	struct FSkeletonCallbackHandles
	{
		/** Editable skeleton interface used to register/unregister notifies callbacks. */
		TSharedPtr<IEditableSkeleton> EditableSkeleton;

		/** Handle for the curve metadata changed callback. */
		FDelegateHandle CurveMetaDataChangedHandle;

		/** The skeleton that was bound. Stored separately so unbinding is safe even if the MirrorTable's Skeleton property has already changed. */
		TWeakObjectPtr<USkeleton> BoundSkeleton;
	};

	/** Binds skeleton change callbacks for the given MirrorTable. Unbinds any existing callbacks first if the MirrorTable was previously bound. */
	void BindSkeletonCallbacks(UMirrorDataTable* MirrorTable);

	/** Unbinds all skeleton change callbacks for the given MirrorTable and removes its entry from ActiveSkeletonCallbacks. */
	void UnbindSkeletonCallbacks(UMirrorDataTable* MirrorTable);

	/** Called when any asset is opened in the editor. Binds skeleton callbacks if the asset is a mirror data table. */
	void OnAssetOpenedInEditor(UObject* Asset, IAssetEditorInstance* EditorInstance);

	/** Called when a property changes on any UObject. Rebinds skeleton callbacks when the Skeleton property changes on a mirror data table. */
	void OnPropertyChanged(UObject* Object, FPropertyChangedEvent& PropertyChangedEvent);

	/** Per-MirrorTable skeleton callback state for all actively-edited mirror data tables. */
	TMap<TWeakObjectPtr<UMirrorDataTable>, FSkeletonCallbackHandles> ActiveSkeletonCallbacks;

	/** Handle for the UAssetEditorSubsystem::OnAssetOpenedInEditor delegate. */
	FDelegateHandle OnAssetOpenedHandle;

	/** Handle for the FCoreUObjectDelegates::OnObjectPropertyChanged delegate. */
	FDelegateHandle OnObjectPropertyChangedHandle;
};
