// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Animation/MirrorDataTable.h"

struct FScopedSlowTask;

class FMirrorDataTableValidator
{
	public:
	
	/** Used to record an issue when validating a Mirror Data Table. */
	struct FIssue
	{
		enum class ESeverity : uint8
		{
			Info,
			Warning,
			Error,
		};
	
		/** Determines the scale of the issue*/
		ESeverity Severity = ESeverity::Info;
		
		/** Summary of what went wrong. */
		FText Message;
		
		/** Row where the issue is present. */
		FName RowName = NAME_None;
	};
	
	/** Used to record the results from validating a Mirror Data Table. */
	struct FResult
	{
		/** Record of everything that is not properly setup. */
		TArray<FIssue> Issues;
	
		/** Do we have any issues at all? */
		bool HasErrors() const
		{
			for (const FIssue& Issue : Issues)
			{
				if (Issue.Severity == FIssue::ESeverity::Error)
				{
					return true;
				}
			}
		
			return false;
		}
	
		/** Do we have any warnings at all? */
		bool HasWarnings() const
		{
			for (const FIssue& Issue : Issues)
			{
				if (Issue.Severity == FIssue::ESeverity::Warning)
				{
					return true;
				}
			}
		
			return false;
		}
	
		/** Records an issue that occurred during validation. */
		void AddIssue(FIssue::ESeverity Severity, const FText& Message, FName RowName = NAME_None)
		{
			FIssue& Issue = Issues.AddDefaulted_GetRef();
			Issue.Severity = Severity;
			Issue.Message = Message;
			Issue.RowName = RowName;
		}
	};
	
	/** Looks for any issues with the provided data table and record them. */
	static FResult Validate(const UMirrorDataTable& MirrorDataTable);
	
	private:
	
	/** Checks for missing or stale skeleton entries. */
	static void ValidateSkeletonBackedEntries(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress);
	
	/** Catches duplicate logical source entries of the same row type. */
	static void ValidateDuplicateMappings(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress);
	
	/** Checks for two entries mapping to the same mirror entry causing the operation to no be invertible. Ex. A -> B and C -> B. */
	static void ValidateReverseMappings(const UMirrorDataTable& MirrorDataTable, FResult& OutResult, FScopedSlowTask& InOutProgress);
};
